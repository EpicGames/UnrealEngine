{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "CryptoKeys",
	"Description": "",
	"Category": "Other",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": false,
	"IsBetaVersion": false,
	"Installed": false,
	"EnabledByDefault": true,
	"Modules": [
		{
			"Name": "CryptoKeys",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Mac",
				"Win64",
				"Linux"
			]
		},
		{
			"Name": "CryptoKeysOpenSSL",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Mac",
				"Win64",
				"Linux"
			]
		}
	]
}