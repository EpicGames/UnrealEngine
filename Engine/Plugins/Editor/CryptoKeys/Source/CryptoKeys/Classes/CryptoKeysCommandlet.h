// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Commandlets/Commandlet.h"
#include "CryptoKeysCommandlet.generated.h"

#define UE_API CRYPTOKEYS_API

/**
* Commandlet used to configure project encryption settings
*/
UCLASS(MinimalAPI)
class UCryptoKeysCommandlet : public UCommandlet
{
	GENERATED_UCLASS_BODY()

		//~ Begin UCommandlet Interface
		UE_API virtual int32 Main(const FString& Params) override;
	//~ End UCommandlet Interface
};

#undef UE_API
