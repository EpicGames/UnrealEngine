// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CryptoKeysSettings.generated.h"

#define UE_API CRYPTOKEYS_API


/**
* UStruct representing a named encryption key
*/
USTRUCT()
struct FCryptoEncryptionKey
{
	GENERATED_BODY()

	UPROPERTY(config, VisibleAnywhere, Category = Encryption)
	FGuid Guid;

	UPROPERTY(config, EditAnywhere, Category = Encryption)
	FString Name;

	UPROPERTY(config, VisibleAnywhere, Category = Encryption)
	FString Key;
};

/**
* Implements the settings for imported Paper2D assets, such as sprite sheet textures.
*/
UCLASS(MinimalAPI, config = Crypto, defaultconfig)
class UCryptoKeysSettings : public UObject
{
	GENERATED_BODY()

public:

	UE_API UCryptoKeysSettings();

	//~ Begin UObject Interface
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent);
	//~ End UObject Interface

	bool IsEncryptionEnabled() const
	{
		return EncryptionKey.Len() > 0 && (bEncryptAllAssetFiles || bEncryptPakIndex || bEncryptPakIniFiles || bEncryptUAssetFiles);
	}

	bool IsSigningEnabled() const
	{
		return bEnablePakSigning && SigningModulus.Len() > 0 && SigningPrivateExponent.Len() > 0 && SigningPublicExponent.Len() > 0;
	}

	// The default encryption key used to protect pak files
	UPROPERTY(config, VisibleAnywhere, Category = Encryption)
	FString EncryptionKey;

	// Secondary encryption keys that can be selected for use on different assets. Games are required to make these keys available to the pak platform file at runtime in order to access the data they protect.
	UPROPERTY(config, EditAnywhere, Category = Encryption)
	TArray<FCryptoEncryptionKey> SecondaryEncryptionKeys;

	// Encrypts all ini files in the pak. Gives security to the most common sources of mineable information, with minimal runtime IO cost
	UPROPERTY(config, EditAnywhere, Category = Encryption)
	bool bEncryptPakIniFiles;

	// Encrypt the pak index, making it impossible to use unrealpak to manipulate the pak file without the encryption key
	UPROPERTY(config, EditAnywhere, Category = Encryption)
	bool bEncryptPakIndex;

	// Encrypts the uasset file in cooked data. Less runtime IO cost, and protection to package header information, including most string data, but still leaves the bulk of the data unencrypted. 
	UPROPERTY(config, EditAnywhere, Category = Encryption)
	bool bEncryptUAssetFiles;

	// Encrypt all files in the pak file. Secure, but will cause some slowdown to runtime IO performance, and high entropy to packaged data which will be bad for patching
	UPROPERTY(config, EditAnywhere, Category = Encryption)
	bool bEncryptAllAssetFiles;

	// The RSA key public exponent used for signing a pak file
	UPROPERTY(config, VisibleAnywhere, Category = Signing)
	FString SigningPublicExponent;

	// The RSA key modulus used for signing a pak file
	UPROPERTY(config, VisibleAnywhere, Category = Signing)
	FString SigningModulus;

	// The RSA key private exponent used for signing a pak file
	UPROPERTY(config, VisibleAnywhere, Category = Signing)
	FString SigningPrivateExponent;

	// Enable signing of pak files, to prevent tampering of the data
	UPROPERTY(config, EditAnywhere, Category = Signing)
	bool bEnablePakSigning;
};

#undef UE_API
