// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class CryptoKeys : ModuleRules
{
	public CryptoKeys(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"CoreUObject",
				"ApplicationCore",
				"Engine",
				"EditorFramework",
				"UnrealEd",
				"CryptoKeysOpenSSL",
				"Slate",
				"SlateCore",
				"GameProjectGeneration",
				"DeveloperToolSettings"
		});
	}
}
