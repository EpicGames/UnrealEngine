// Copyright Epic Games, Inc. All Rights Reserved.

#include "CryptoKeys.h"
#include "Modules/ModuleManager.h"
#include "ISettingsModule.h"
#include "PropertyEditorModule.h"
#include "CryptoKeysSettings.h"
#include "CryptoKeysHelpers.h"
#include "CryptoKeysSettingsDetails.h"
#include "CryptoKeysProjectBuildMutatorFeature.h"
#include "Features/IModularFeatures.h"

#define LOCTEXT_NAMESPACE "CryptoKeysModule"

class FCryptoKeysModule : public IModuleInterface
{
public:

	/** IModuleInterface implementation */
	virtual void StartupModule() override
	{
		RegisterSettings();

		FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.RegisterCustomClassLayout(UCryptoKeysSettings::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FCryptoKeysSettingsDetails::MakeInstance));

		IModularFeatures::Get().RegisterModularFeature(PROJECT_BUILD_MUTATOR_FEATURE, &ProjectBuildMutator);
	}

	virtual void ShutdownModule() override
	{
		if (UObjectInitialized())
		{
			UnregisterSettings();

			FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
			PropertyModule.UnregisterCustomClassLayout(UCryptoKeysSettings::StaticClass()->GetFName());

			IModularFeatures::Get().UnregisterModularFeature(PROJECT_BUILD_MUTATOR_FEATURE, &ProjectBuildMutator);
		}
	}

	void RegisterSettings()
	{
		if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
		{
			SettingsModule->RegisterSettings("Project", "Project", "Encryption",
				LOCTEXT("CryptoSettingsName", "Encryption"),
				LOCTEXT("CryptoSettingsDescription", "Configure the project encryption keys"),
				GetMutableDefault<UCryptoKeysSettings>());
		}
	}

	void UnregisterSettings()
	{
		if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
		{
			SettingsModule->UnregisterSettings("Project", "Project", "Encryption");
		}
	}

private:

	FCryptoKeysProjectBuildMutatorFeature ProjectBuildMutator;
};

namespace CryptoKeys
{
	void GenerateEncryptionKey(FString& OutBase64Key)
	{
		CryptoKeysHelpers::GenerateEncryptionKey(OutBase64Key);
	}
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FCryptoKeysModule, CryptoKeys)
