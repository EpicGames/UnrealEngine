// Copyright Epic Games, Inc. All Rights Reserved.

#include "CryptoKeysCommandlet.h"
#include "CryptoKeysSettings.h"
#include "CryptoKeysHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CryptoKeysCommandlet)

DEFINE_LOG_CATEGORY_STATIC(LogCryptoKeysCommandlet, Log, All);

UCryptoKeysCommandlet::UCryptoKeysCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

}

int32 UCryptoKeysCommandlet::Main(const FString& InParams)
{
	bool bUpdateAllKeys = FParse::Param(*InParams, TEXT("updateallkeys"));
	bool bUpdateEncryptionKey = bUpdateAllKeys || FParse::Param(*InParams, TEXT("updateencryptionkey"));
	bool bUpdateSigningKey = bUpdateAllKeys || FParse::Param(*InParams, TEXT("updatesigningkey"));
	bool bTestSigningKeyGeneration = FParse::Param(*InParams, TEXT("testsigningkeygen")); 

	if (bUpdateEncryptionKey || bUpdateSigningKey)
	{
		UCryptoKeysSettings* Settings = GetMutableDefault<UCryptoKeysSettings>();
		bool Result;

		if (bUpdateEncryptionKey)
		{
			Result = CryptoKeysHelpers::GenerateEncryptionKey(Settings->EncryptionKey);
			check(Result);
		}

		if (bUpdateSigningKey)
		{
			Result = CryptoKeysHelpers::GenerateSigningKey(Settings->SigningPublicExponent, Settings->SigningPrivateExponent, Settings->SigningModulus);
			check(Result);
		}

		Settings->TryUpdateDefaultConfigFile();
	}

	if (bTestSigningKeyGeneration)
	{
		TArray<FString> PublicExponents, PrivateExponents, Moduli;

		static const int32 NumLoops = INT32_MAX;
		for (int32 LoopCount = 0; LoopCount < NumLoops; ++LoopCount)
		{
			UE_LOGF(LogCryptoKeysCommandlet, Display, "Key generation test [%i/%i]", LoopCount + 1, NumLoops);
			FString PublicExponent, PrivateExponent, Modulus;
			bool bResult = CryptoKeysHelpers::GenerateSigningKey(PublicExponent, PrivateExponent, Modulus);
			check(bResult);

			check(!PublicExponents.Contains(PublicExponent));
			check(!PrivateExponents.Contains(PrivateExponent));
			check(!Moduli.Contains(Modulus));

			PublicExponents.Add(PublicExponent);
			PrivateExponents.Add(PrivateExponent);
			Moduli.Add(Modulus);
		}
	}

	return 0;
}

