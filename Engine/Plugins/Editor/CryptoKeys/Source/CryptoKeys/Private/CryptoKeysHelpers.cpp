// Copyright Epic Games, Inc. All Rights Reserved.

#include "CryptoKeysHelpers.h"
#include "CryptoKeysOpenSSL.h"

#include "Logging/LogMacros.h"
#include "Misc/Base64.h"


DEFINE_LOG_CATEGORY_STATIC(LogCryptoKeys, Log, All);

namespace CryptoKeysHelpers
{
	bool GenerateEncryptionKey(FString& OutEncryptionKey)
	{
		bool bResult = false;

		TArray<uint8> NewEncryptionKey;
		if (CryptoKeysOpenSSL::GenerateNewEncryptionKey(NewEncryptionKey))
		{
			OutEncryptionKey = FBase64::Encode(NewEncryptionKey);
			bResult = true;
		}

		return bResult;
	}

	bool GenerateSigningKey(FString& OutPublicExponent, FString& OutPrivateExponent, FString& OutModulus, int32 NumKeyBits)
	{
		bool bResult = false;

		TArray<uint8> PublicExponent, PrivateExponent, Modulus;
		if (CryptoKeysOpenSSL::GenerateNewSigningKey(PublicExponent, PrivateExponent, Modulus, NumKeyBits))
		{
			OutPublicExponent = FBase64::Encode(PublicExponent);
			OutPrivateExponent = FBase64::Encode(PrivateExponent);
			OutModulus = FBase64::Encode(Modulus);
			bResult = true;
		}

		return bResult;
	}
}
