// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

class FString;

namespace CryptoKeysHelpers
{
	/**
	Generates a new AES key
	@param OutEncryptionKey - Assigned the base64 encoded representation of the new key
	@returns true if key generation succeeded, false otherwise
	*/
	bool GenerateEncryptionKey(FString& OutEncryptionKey);

	/**
	Generates a new RSA signing key
	@param OutPublicExponent 	- Assigned the base64 encoded representation of the RSA public exponent
	@param OutPrivateExponent 	- Assigned the base64 encoded representation of the RSA private exponent
	@param OutModulus		 Assigned the base64 encoded representation of the RSA modulus
	@param InNumKeyBits 		- How many bits to use for the RSA key
	@returns true if key generation succeeded, false otherwise
	*/
	bool GenerateSigningKey(FString& OutPublicExponent, FString& OutPrivateExponent, FString& OutModulus, int32 NumKeyBits = 4096);
}