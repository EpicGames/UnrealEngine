// Copyright Epic Games, Inc. All Rights Reserved.
#include "CryptoKeysProjectBuildMutatorFeature.h"
#include "CryptoKeysSettings.h"

#define LOCTEXT_NAMESPACE "CryptoKeys"

bool FCryptoKeysProjectBuildMutatorFeature ::RequiresProjectBuild(const FName& InPlatformInfoName, FText& OutReason) const
{
	UCryptoKeysSettings* Settings = GetMutableDefault<UCryptoKeysSettings>();
	if (Settings->IsEncryptionEnabled() || Settings->IsSigningEnabled())
	{
		OutReason = LOCTEXT("BuildMutator_BuildRequired", "encryption/signing enabled");
		return true;
	}
	return false;
}

#undef LOCTEXT_NAMESPACE
