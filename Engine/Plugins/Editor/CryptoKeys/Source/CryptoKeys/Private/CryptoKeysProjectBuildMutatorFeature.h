// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/IProjectBuildMutatorFeature.h"

class FCryptoKeysProjectBuildMutatorFeature : public IProjectBuildMutatorFeature
{
public:
	virtual bool RequiresProjectBuild(const FName& InPlatformInfoName, FText& OutReason) const override;
};