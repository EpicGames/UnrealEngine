// Copyright Epic Games, Inc. All Rights Reserved.
#include "CryptoKeysSettings.h"
#include "CryptoKeysHelpers.h"
#include "Settings/ProjectPackagingSettings.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ConfigContext.h"
#include "Misc/Base64.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CryptoKeysSettings)

UCryptoKeysSettings::UCryptoKeysSettings()
{
	// Migrate any settings from the old ini files if they exist
	UProjectPackagingSettings* ProjectPackagingSettings = GetMutableDefault<UProjectPackagingSettings>();
	if (ProjectPackagingSettings)
	{
		bEncryptPakIniFiles = ProjectPackagingSettings->bEncryptIniFiles_DEPRECATED;
		bEncryptPakIndex = ProjectPackagingSettings->bEncryptPakIndex_DEPRECATED;

		if (GConfig->IsReadyForUse())
		{
			FConfigFile EncryptionConfig;
			FConfigContext::ReadIntoLocalFile(EncryptionConfig).Load(TEXT("Encryption"));

			FString OldEncryptionKey;
			if (EncryptionConfig.GetString(TEXT("Core.Encryption"), TEXT("aes.key"), OldEncryptionKey))
			{
				EncryptionKey = FBase64::Encode(OldEncryptionKey);
			}

			FString OldSigningModulus, OldSigningPublicExponent, OldSigningPrivateExponent;

			bEnablePakSigning = EncryptionConfig.GetString(TEXT("Core.Encryption"), TEXT("rsa.privateexp"), OldSigningPrivateExponent)
				&& EncryptionConfig.GetString(TEXT("Core.Encryption"), TEXT("rsa.publicexp"), OldSigningPublicExponent)
				&& EncryptionConfig.GetString(TEXT("Core.Encryption"), TEXT("rsa.modulus"), OldSigningModulus);

			if (bEnablePakSigning)
			{
				SigningModulus = FBase64::Encode(OldSigningModulus);
				SigningPublicExponent = FBase64::Encode(OldSigningPublicExponent);
				SigningPrivateExponent = FBase64::Encode(OldSigningPrivateExponent);
			}
		}
	}
}

void UCryptoKeysSettings::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetName() == TEXT("SecondaryEncryptionKeys"))
	{
		if (PropertyChangedEvent.ChangeType == EPropertyChangeType::ArrayAdd)
		{
			int32 Index = PropertyChangedEvent.GetArrayIndex(TEXT("SecondaryEncryptionKeys"));
			CryptoKeysHelpers::GenerateEncryptionKey(SecondaryEncryptionKeys[Index].Key);

			int32 Number = 1;
			FString NewName = FString::Printf(TEXT("New Encryption Key %d"), Number++);
			while (SecondaryEncryptionKeys.FindByPredicate([NewName](const FCryptoEncryptionKey& Key) { return Key.Name == NewName; }) != nullptr)
			{
				NewName = FString::Printf(TEXT("New Encryption Key %d"), Number++);
			}

			SecondaryEncryptionKeys[Index].Name = NewName;
			SecondaryEncryptionKeys[Index].Guid = FGuid::NewGuid();
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
