// Copyright Epic Games, Inc. All Rights Reserved.

#include "CryptoKeysSettingsDetails.h"
#include "CryptoKeysSettings.h"
#include "CryptoKeysHelpers.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "CryptoKeysSettingsDetails"

TSharedRef<IDetailCustomization> FCryptoKeysSettingsDetails::MakeInstance()
{
	return MakeShareable(new FCryptoKeysSettingsDetails);
}

void FCryptoKeysSettingsDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);
	check(ObjectsBeingCustomized.Num() == 1);
	TWeakObjectPtr<UCryptoKeysSettings> Settings = Cast<UCryptoKeysSettings>(ObjectsBeingCustomized[0].Get());

	IDetailCategoryBuilder& EncryptionCategory = DetailBuilder.EditCategory("Encryption");

	EncryptionCategory.AddCustomRow(LOCTEXT("EncryptionKeyGenerator", "EncryptionKeyGenerator"))
		.ValueContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(5)
			.AutoWidth()
			[
				SNew(SButton)
				.Text(LOCTEXT("GenerateEncryptionKey", "Generate New Encryption Key"))
				.ToolTipText(LOCTEXT("GenerateEncryptionKey_Tooltip", "Generates a new encryption key"))
				.OnClicked_Lambda([this, Settings]()
				{
					if (Settings.IsValid() && CryptoKeysHelpers::GenerateEncryptionKey(Settings->EncryptionKey))
					{
						Settings->TryUpdateDefaultConfigFile();
					}
					else
					{
						// toast?
					}

					return(FReply::Handled());
				})
			]
			+ SHorizontalBox::Slot()
			.Padding(5)
			.AutoWidth()
			[
				SNew(SButton)
				.Text(LOCTEXT("ClearEncryptionKey", "Clear Encryption Key"))
				.ToolTipText(LOCTEXT("ClearEncryptionKey_Tooltip", "Clears the current encryption key"))
				.OnClicked_Lambda([this, Settings]()
				{
					Settings->EncryptionKey = TEXT("");
					Settings->TryUpdateDefaultConfigFile();
					return(FReply::Handled());
				})
			]
		];

	IDetailCategoryBuilder& SigningCategory = DetailBuilder.EditCategory("Signing");

	SigningCategory.AddCustomRow(LOCTEXT("SigningKeyGenerator", "SigningKeyGenerator"))
		.ValueContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(5)
			.AutoWidth()
			[
				SNew(SButton)
				.Text(LOCTEXT("GenerateSigningKey", "Generate New Signing Keys"))
				.ToolTipText(LOCTEXT("GenerateSigningKey_Tooltip", "Generates a new signing key"))
				.OnClicked_Lambda([this, Settings]()
				{
					if (Settings.IsValid() && CryptoKeysHelpers::GenerateSigningKey(Settings->SigningPublicExponent, Settings->SigningPrivateExponent, Settings->SigningModulus))
					{
						Settings->TryUpdateDefaultConfigFile();
					}
					else
					{
						// toast?
					}
					return(FReply::Handled());
				})
			]
			+ SHorizontalBox::Slot()
			.Padding(5)
			.AutoWidth()
			[
				SNew(SButton)
				.Text(LOCTEXT("ClearSigningKey", "Clear Signing Key"))
				.ToolTipText(LOCTEXT("ClearSigningKey_Tooltip", "Clears the current signing key"))
				.OnClicked_Lambda([this, Settings]()
				{
					Settings->SigningModulus = TEXT("");
					Settings->SigningPublicExponent = TEXT("");
					Settings->SigningPrivateExponent = TEXT("");
					Settings->TryUpdateDefaultConfigFile();
					return(FReply::Handled());
				})
			]
		];
}

#undef LOCTEXT_NAMESPACE
