// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

class FString;

namespace CryptoKeys
{
	CRYPTOKEYS_API void GenerateEncryptionKey(FString& OutBase64Key);
}
