// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class CryptoKeysOpenSSL : ModuleRules
{
	public CryptoKeysOpenSSL(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateIncludePathModuleNames.Add("CryptoKeys");

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"OpenSSL"
			}
		);

		AddEngineThirdPartyPrivateStaticDependencies(Target, "OpenSSL");
	}
}