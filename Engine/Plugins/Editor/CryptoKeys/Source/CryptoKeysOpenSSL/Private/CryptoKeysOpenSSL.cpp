// Copyright Epic Games, Inc. All Rights Reserved.
#include "CryptoKeysOpenSSL.h"
#include "Modules/ModuleManager.h"

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#endif

#include <openssl/opensslv.h>
#include <openssl/rsa.h>
#include <openssl/ossl_typ.h>
#include <openssl/rand.h>
#include <openssl/bn.h>

#if PLATFORM_WINDOWS
#include "Windows/HideWindowsPlatformTypes.h"
#endif

DEFINE_LOG_CATEGORY_STATIC(LogCryptoKeys, Log, All);

#if !defined(OPENSSL_VERSION_NUMBER) || OPENSSL_VERSION_NUMBER < 0x10100000L
#define USE_LEGACY_OPENSSL 1
#else
#define USE_LEGACY_OPENSSL 0
#endif

namespace CryptoKeysOpenSSL
{
	bool GenerateNewEncryptionKey(TArray<uint8>& OutKey)
	{
		OutKey.Empty(32);
		OutKey.AddUninitialized(32);
		bool bResult = !!RAND_bytes(&OutKey[0], 32);
		if (!bResult)
		{
			OutKey.Empty();
		}
		return bResult;
	}

	void BigNumToArray(const BIGNUM* InNum, TArray<uint8>& OutBytes, int32 InKeySize)
	{
		int32 NumBytes = BN_num_bytes(InNum);
		check(NumBytes <= InKeySize);
		OutBytes.SetNumZeroed(NumBytes);

		BN_bn2bin(InNum, OutBytes.GetData());
		Algo::Reverse(OutBytes);
	}

	bool GenerateNewSigningKey(TArray<uint8>& OutPublicExponent, TArray<uint8>& OutPrivateExponent, TArray<uint8>& OutModulus, int32 InNumKeyBits)
	{
		int32 KeySize = InNumKeyBits;
		int32 KeySizeInBytes = InNumKeyBits / 8;

		RSA* RSAKey = RSA_new();
		BIGNUM* E = BN_new();
		BN_set_word(E, RSA_F4);
		RSA_generate_key_ex(RSAKey, KeySize, E, nullptr);

#if USE_LEGACY_OPENSSL
		const BIGNUM* PublicModulus = RSAKey->n;
		const BIGNUM* PublicExponent = RSAKey->e;
		const BIGNUM* PrivateExponent = RSAKey->d;
#else
		const BIGNUM* PublicModulus = RSA_get0_n(RSAKey);
		const BIGNUM* PublicExponent = RSA_get0_e(RSAKey);
		const BIGNUM* PrivateExponent = RSA_get0_d(RSAKey);
#endif

		BigNumToArray(PublicModulus, OutModulus, KeySizeInBytes);
		BigNumToArray(PublicExponent, OutPublicExponent, KeySizeInBytes);
		BigNumToArray(PrivateExponent, OutPrivateExponent, KeySizeInBytes);
		
		RSA_free(RSAKey);

		return true;
	}
}

IMPLEMENT_MODULE(FDefaultModuleImpl, CryptoKeysOpenSSL)
