// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/ContainersFwd.h" // IWYU pragma: keep

namespace CryptoKeysOpenSSL
{
	bool CRYPTOKEYSOPENSSL_API GenerateNewEncryptionKey(TArray<uint8>& OutKey);
	bool CRYPTOKEYSOPENSSL_API GenerateNewSigningKey(TArray<uint8>& OutPublicExponent, TArray<uint8>& OutPrivateExponent, TArray<uint8>& OutModulus, int32 InNumKeyBits = 2048);
}
